// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GizmoSettings : ModuleRules
	{
		public GizmoSettings(ReadOnlyTargetRules Target) : base(Target)
		{
			// Enable truncation warnings in this module
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "DeveloperSettings",
                    "Engine",
                    "EditorInteractiveToolsFramework"
				}
			);
		}
	}
}
