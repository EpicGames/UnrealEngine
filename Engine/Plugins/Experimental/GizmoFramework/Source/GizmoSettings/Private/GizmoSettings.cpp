// Copyright Epic Games, Inc. All Rights Reserved.

#include "GizmoSettings.h"
#include "EditorInteractiveGizmoManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoSettings)

void UGizmoSettings::PostInitProperties()
{
	Super::PostInitProperties();

	bool bConfigNeedsUpdate = false;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bEnableNewGizmos_DEPRECATED)
	{
		UEditorInteractiveGizmoManager::SetUsesNewTRSGizmos(bEnableNewGizmos_DEPRECATED);
		bEnableNewGizmos_DEPRECATED = false;

		bConfigNeedsUpdate = true;
	}

	FGizmosParameters DefaultGizmosParameters;
	if (GizmoParameters_DEPRECATED.bAdditiveIndirectAxes != DefaultGizmosParameters.bAdditiveIndirectAxes ||
		GizmoParameters_DEPRECATED.bEnableExplicit != DefaultGizmosParameters.bEnableExplicit ||
		GizmoParameters_DEPRECATED.RotateMode != DefaultGizmosParameters.RotateMode)
	{
		UEditorInteractiveGizmoManager::SetGizmosParameters(GizmoParameters_DEPRECATED);

		bConfigNeedsUpdate = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (bConfigNeedsUpdate)
	{
		SaveConfig();
	}
}
