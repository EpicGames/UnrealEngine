// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

// Dummy module, currently only used by Viewport Toolbar to show/hide the "Use Experimental Gizmos" entry.
class FGizmoSettingsModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FGizmoSettingsModule, GizmoSettings)
