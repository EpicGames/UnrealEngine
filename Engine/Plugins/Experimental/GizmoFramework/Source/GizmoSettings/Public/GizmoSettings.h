// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "EditorGizmos/TransformGizmo.h"

#include "GizmoSettings.generated.h"

UCLASS(config=EditorPerProjectUserSettings, meta = (DisplayName = "New TRS Gizmo"))
class UE_DEPRECATED(5.6, "New Gizmos settings can now be found in UTransformGizmoEditorSettings") GIZMOSETTINGS_API UGizmoSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Enable/Disable New TRS Gizmos across the editor. */
	UE_DEPRECATED(5.6, "UGizmoSettings::bEnableNewGizmos is deprecated. See UTransformGizmoEditorSettings::bUseExperimentalGizmo")
	UPROPERTY(config)
	bool bEnableNewGizmos_DEPRECATED = false;

	/** Change the current gizmos parameters. */
	UE_DEPRECATED(5.6, "UGizmoSettings::GizmoParameters is deprecated. See UTransformGizmoEditorSettings::GizmoParameters")
	UPROPERTY(config)
	FGizmosParameters GizmoParameters_DEPRECATED;

	virtual void PostInitProperties() override;

	/** We don't want to show this in the Editor Preferences list */
	virtual bool SupportsAutoRegistration() const override { return false; }
};
